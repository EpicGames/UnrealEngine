// Copyright Epic Games, Inc. All Rights Reserved.

#include "ManifestUpdateHelper.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FManifestUpdateHelper"

/////////////////////////////////////////////////////
// FManifestUpdateHelper


FManifestUpdateHelper::FManifestUpdateHelper(const FString& InFilename)
: bManifestDirty(false)
{
	if (!FFileHelper::LoadFileToString(ManifestString, *InFilename))
	{
		WriteError(FText::Format(LOCTEXT("FailedToReadManifest", "Failed to load '{0}'"), FText::FromString(InFilename)));
	}
}

bool FManifestUpdateHelper::Finalize(const FString& TargetFilename, bool bShowNotifyOnFailure, FFileHelper::EEncodingOptions EncodingOption)
{
	if (bManifestDirty)
	{
		if (!FFileHelper::SaveStringToFile(ManifestString, *TargetFilename, EncodingOption))
		{
			WriteError(FText::Format(LOCTEXT("FailedToWriteManifest", "Failed to save '{0}'"), FText::FromString(TargetFilename)));
		}
	}

	if (bShowNotifyOnFailure && !FirstErrorMessage.IsEmpty())
	{
		FNotificationInfo Info(FirstErrorMessage);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	return !FirstErrorMessage.IsEmpty();
}

void FManifestUpdateHelper::WriteError(FText NewError)
{
	if (FirstErrorMessage.IsEmpty())
	{
		FirstErrorMessage = NewError;
	}

	UE_LOGF(LogInit, Warning, "Error during platform manifest modification: %ls", *NewError.ToString());
}

bool FManifestUpdateHelper::HasKey(const FString& MatchPrefix)
{
	const int32 PrefixPos = ManifestString.Find(MatchPrefix, ESearchCase::CaseSensitive);
	return (PrefixPos != INDEX_NONE);
}

void FManifestUpdateHelper::ReplaceKey(const FString& MatchPrefix, const FString& MatchSuffix, const FString& NewInfix)
{
	if (ReplaceStringPortion(ManifestString, MatchPrefix, MatchSuffix, NewInfix))
	{
		bManifestDirty = true;
	}
	else
	{
		WriteError(FText::Format(LOCTEXT("FailedToReplaceManifestString", "Failed to find prefix '{0}' or suffix '{1}' while writing '{2}'"),
			FText::FromString(MatchPrefix), FText::FromString(MatchSuffix), FText::FromString(NewInfix)));
	}
}

bool FManifestUpdateHelper::ReplaceStringPortion(FString& InOutString, const FString& MatchPrefix, const FString& MatchSuffix, const FString& NewInfix)
{
	const int32 PrefixPos = InOutString.Find(MatchPrefix, ESearchCase::CaseSensitive);
	if (PrefixPos != INDEX_NONE)
	{
		const int32 StartPos = PrefixPos + MatchPrefix.Len();
		const int32 StopPos = InOutString.Find(MatchSuffix, ESearchCase::CaseSensitive, ESearchDir::FromStart, StartPos);

		if (StopPos != INDEX_NONE)
		{
			InOutString = InOutString.Left(StartPos) + NewInfix + InOutString.RightChop(StopPos);
			return true;
		}
	}

	return false;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
