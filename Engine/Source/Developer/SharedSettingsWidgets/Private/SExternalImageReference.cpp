// Copyright Epic Games, Inc. All Rights Reserved.

#include "SExternalImageReference.h"
#include "HAL/FileManager.h"

#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SExternalImageReference"


void SExternalImageReference::Construct(const FArguments& InArgs, const FString& InBaseFilename, const FString& InOverrideFilename)
{
	FileDescription = InArgs._FileDescription;
	OnPreExternalImageCopy = InArgs._OnPreExternalImageCopy;
	OnPostExternalImageCopy = InArgs._OnPostExternalImageCopy;
	BaseFilename = InBaseFilename;
	OverrideFilename = InOverrideFilename;
	bDeleteTargetWhenDefaultChosen = InArgs._DeleteTargetWhenDefaultChosen;
	bDeletePreviousTargetWhenExtensionChanges = InArgs._DeletePreviousTargetWhenExtensionChanges;
	Extensions = InArgs._FileExtensions;

	FString BaseFilenameExtension = FPaths::GetExtension(BaseFilename);
	if (!Extensions.Contains(BaseFilenameExtension))
	{
		Extensions.Add(BaseFilenameExtension);
	}

	FExternalImagePickerConfiguration ImageReferenceConfig;
	ImageReferenceConfig.TargetImagePath = InOverrideFilename;
	ImageReferenceConfig.DefaultImagePath = InBaseFilename;
	ImageReferenceConfig.OnExternalImagePicked = FOnExternalImagePicked::CreateSP(this, &SExternalImageReference::HandleExternalImagePicked);
	ImageReferenceConfig.RequiredImageDimensions = InArgs._RequiredSize;
	ImageReferenceConfig.bRequiresSpecificSize = InArgs._RequiredSize.X >= 0;
	ImageReferenceConfig.MaxDisplayedImageDimensions = InArgs._MaxDisplaySize;
	ImageReferenceConfig.OnGetPickerPath = InArgs._OnGetPickerPath;
	ImageReferenceConfig.FileExtensions = InArgs._FileExtensions;
	ImageReferenceConfig.GenerateImageVisibility = InArgs._GenerateImageVisibility;
	ImageReferenceConfig.GenerateImageToolTipText = InArgs._GenerateImageToolTipText;
	ImageReferenceConfig.OnGenerateImageClicked = InArgs._OnGenerateImageClicked;
  
	ChildSlot
	[
		IExternalImagePickerModule::Get().MakeEditorWidget(ImageReferenceConfig)
	];
}


bool SExternalImageReference::HandleExternalImagePicked(const FString& InChosenImage, const FString& InTargetImage)
{
	const FString TargetImagePathNoExtension = FPaths::GetPath(InTargetImage) / FPaths::GetBaseFilename(InTargetImage) + TEXT(".");
	
	if (bDeleteTargetWhenDefaultChosen && InChosenImage == BaseFilename)
	{
		for (FString& Ex : Extensions)
		{
			const FString TargetImagePath = TargetImagePathNoExtension + Ex;
			IFileManager& FileManager = IFileManager::Get();

			if (!FileManager.FileExists(*TargetImagePath))
			{
				continue;
			}

			// We elect to remove the target image completely if the default image is chosen (thus allowing us to distinguish the default from a non-default image)
			if (ISourceControlModule::Get().IsEnabled())
			{
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

				const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(TargetImagePath, EStateCacheUsage::ForceUpdate);
				const bool bIsSourceControlled = SourceControlState.IsValid() && SourceControlState->IsSourceControlled();

				if (bIsSourceControlled)
				{
					// The file is managed by source control. Delete it through there.
					TArray<FString> DeleteFilenames;
					DeleteFilenames.Add(TargetImagePath);

					// Revert the file if it is checked out
					const bool bIsAdded = SourceControlState->IsAdded();
					if (SourceControlState->IsCheckedOut() || bIsAdded || SourceControlState->IsDeleted())
					{
						SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), DeleteFilenames);
					}

					// If it wasn't already marked as an add, we can ask the source control provider to delete the file
					if (!bIsAdded)
					{
						// Open the file for delete
						SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), DeleteFilenames);
					}
				}
			}

			const bool bRequireExists = false;
			const bool bEvenIfReadOnly = true;
			const bool bQuiet = true;
			FileManager.Delete(*TargetImagePath, bRequireExists, bEvenIfReadOnly, bQuiet);
		}

		return true;
	}

	if(OnPreExternalImageCopy.IsBound())
	{
		if(!OnPreExternalImageCopy.Execute(InChosenImage))
		{
			return false;
		}
	}

	// New target image file extension from chosen image
	FString NewTargetImage;

	if (FPaths::GetExtension(InTargetImage) != FPaths::GetExtension(InChosenImage))
	{
		if (bDeletePreviousTargetWhenExtensionChanges)
		{
			IFileManager& FileManager = IFileManager::Get();
			const bool bRequireExists = false;
			const bool bEvenIfReadOnly = true;
			const bool bQuiet = true;
			FileManager.Delete(*InTargetImage, bRequireExists, bEvenIfReadOnly, bQuiet);
		}

		NewTargetImage = TargetImagePathNoExtension + FPaths::GetExtension(InChosenImage);
	}
	else
	{
		NewTargetImage = InTargetImage;
	}
	
	FText FailReason;
	if(!SourceControlHelpers::CopyFileUnderSourceControl(NewTargetImage, InChosenImage, LOCTEXT("ImageDescription", "image"), FailReason))
	{
		FNotificationInfo Info(FailReason);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return false;
	}
	
	if(OnPostExternalImageCopy.IsBound())
	{
		if(!OnPostExternalImageCopy.Execute(InChosenImage))
		{
			return false;
		}
	}

	return true;
}


#undef LOCTEXT_NAMESPACE
