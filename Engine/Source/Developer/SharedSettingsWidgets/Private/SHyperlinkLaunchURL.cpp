// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHyperlinkLaunchURL.h"
#include "HAL/PlatformProcess.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "SHyperlinkLaunchURL"

/////////////////////////////////////////////////////
// SHyperlinkLaunchURL

void SHyperlinkLaunchURL::Construct(const FArguments& InArgs, const FString& InDestinationURL)
{
	DestinationURL = InDestinationURL;

	TAttribute<FText> DisplayText;
	if (InArgs._Text.IsBound() || !InArgs._Text.Get().IsEmpty())
	{
		DisplayText = InArgs._Text;
	}
	else
	{
		DisplayText = FText::FromString(DestinationURL);
	}

	ChildSlot
	[
		SNew(SHyperlink)
		.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
		.Text(DisplayText)
		.ToolTipText(InArgs._ToolTipText)
		.OnNavigate(this, &SHyperlinkLaunchURL::OnNavigate)
	];
}

void SHyperlinkLaunchURL::OnNavigate()
{
	FPlatformProcess::LaunchURL(*DestinationURL, NULL, NULL);
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
