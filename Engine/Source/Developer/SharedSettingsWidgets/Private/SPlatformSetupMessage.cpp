// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlatformSetupMessage.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SPlatformSetupMessage"

/////////////////////////////////////////////////////
// SPlatformSetupMessage

TSharedRef<SWidget> SPlatformSetupMessage::MakeRow(FName IconName, FText Message, FText ButtonMessage)
{
	FText Tooltip = FText::Format(LOCTEXT("PlatformSetupTooltip", "Status of platform setup file\n'{0}'"), FText::FromString(TargetFilename));

	TSharedRef<SHorizontalBox> Result = SNew(SHorizontalBox)
		.ToolTipText(Tooltip)

		// Status icon
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(IconName))
		]

		// Notice
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(16.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FLinearColor::White)
			.ShadowColorAndOpacity(FLinearColor::Black)
			.ShadowOffset(FVector2D::UnitVector)
			.Text(Message)
		];


	if (!ButtonMessage.IsEmpty())
	{
		Result->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.OnClicked(this, &SPlatformSetupMessage::OnButtonPressed)
				.Text(ButtonMessage)
			];
	}

	return Result;
}

void SPlatformSetupMessage::Construct(const FArguments& InArgs, const FString& InTargetFilename)
{
	TargetFilename = InTargetFilename;

	OnSetupClicked = InArgs._OnSetupClicked;

	TSharedRef<SWidget> MissingFilesWidget = MakeRow(
		"SettingsEditor.WarningIcon",
		FText::Format(LOCTEXT("MissingFilesText", "Project is not configured for the {0} platform"), InArgs._PlatformName),
		LOCTEXT("MissingFilesButton", "Configure Now"));

	TSharedRef<SWidget> NeedsCheckoutWidget = MakeRow(
		"SettingsEditor.WarningIcon",
		LOCTEXT("NeedsCheckoutText", "Platform files are under revision control"),
		LOCTEXT("NeedsCheckoutButton", "Check Out"));

	TSharedRef<SWidget> ReadOnlyFilesWidget = MakeRow(
		"SettingsEditor.WarningIcon",
		LOCTEXT("ReadOnlyText", "Platform files are read-only or locked"),
		LOCTEXT("ReadOnlyButton", "Make Writable"));

	TSharedRef<SWidget> ReadyToModifyWidget = MakeRow(
		"SettingsEditor.GoodIcon",
		LOCTEXT("ReadyToModifyText", "Platform files are writeable"),
		FText::GetEmpty());

	TSharedRef<SWidget> GettingStatusWidget = MakeRow(
		"SettingsEditor.WarningIcon",
		LOCTEXT("GettingStatusText", "Getting status from source control"),
		FText::GetEmpty());

	ChildSlot
	[
		SNew(SBorder)
		.BorderBackgroundColor(this, &SPlatformSetupMessage::GetBorderColor)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
		.Padding(8.0f)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &SPlatformSetupMessage::GetSetupStateAsInt)

			// Locked slot
			+SWidgetSwitcher::Slot()
			[
				MissingFilesWidget
			]
			+SWidgetSwitcher::Slot()
			[
				NeedsCheckoutWidget
			]
			+SWidgetSwitcher::Slot()
			[
				ReadOnlyFilesWidget
			]
			+SWidgetSwitcher::Slot()
			[
				ReadyToModifyWidget
			]
			+SWidgetSwitcher::Slot()
			[
				GettingStatusWidget
			]
		]
	];

	UpdateCache(true);
}

void SPlatformSetupMessage::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	UpdateCache(false);
}

int32 SPlatformSetupMessage::GetSetupStateAsInt() const
{
	return (int32)CachedSetupState;
}

SPlatformSetupMessage::ESetupState SPlatformSetupMessage::GetSetupStateBasedOnFile(bool bInitStatus)
{
	if (!FPaths::FileExists(TargetFilename))
	{
		return MissingFiles;
	}
	else
	{
		ISourceControlModule& SCC = ISourceControlModule::Get();
		if (SCC.IsEnabled() && SCC.GetProvider().IsAvailable())
		{
			ISourceControlProvider& Provider = SCC.GetProvider();

			if (bInitStatus)
			{
				TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
				if (Provider.Execute(UpdateStatusOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlatformSetupMessage::OnSourceControlOperationComplete)) == ECommandResult::Succeeded)
				{
					return GettingStatus;
				}
			}
			else
			{
				FSourceControlStatePtr SourceControlState = Provider.GetState(TargetFilename, EStateCacheUsage::Use);
				if (SourceControlState.IsValid())
				{
					if (SourceControlState->IsSourceControlled())
					{
						if (SourceControlState->CanCheckout())
						{
							return NeedsCheckout;
						}
					}
					else
					{
						//@TODO: Should we instead try to add the file?
					}
				}
			}
		}
		
		// SCC is disabled or unavailable
		const bool bIsReadOnly = FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*TargetFilename);
		return bIsReadOnly ? ReadOnlyFiles : ReadyToModify;
	}
}

void SPlatformSetupMessage::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	UpdateCache(false); // the SCC cache can be used now
}

void SPlatformSetupMessage::UpdateCache(bool bForceUpdate)
{
	CachedSetupState = GetSetupStateBasedOnFile(bForceUpdate);
}

FSlateColor SPlatformSetupMessage::GetBorderColor() const
{
	switch (CachedSetupState)
	{
	case MissingFiles:
		return FLinearColor(0.8f, 0, 0);
	case ReadyToModify:
		return FLinearColor::Green;
	case ReadOnlyFiles:
	case NeedsCheckout:
	case GettingStatus:
	default:
		return FLinearColor::Yellow;
	}
}

FReply SPlatformSetupMessage::OnButtonPressed()
{
	switch (CachedSetupState)
	{
	case MissingFiles:
		OnSetupClicked.Execute();
		UpdateCache(false);
		break;

	case ReadOnlyFiles:
		{
			if (!FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*TargetFilename, false))
			{
				FText NotificationErrorText = FText::Format(LOCTEXT("FailedToMakeWritable", "Could not make {0} writable."), FText::FromString(TargetFilename));

				FNotificationInfo Info(NotificationErrorText);
				Info.ExpireDuration = 3.0f;

				FSlateNotificationManager::Get().AddNotification(Info);
			}
			UpdateCache(false);
		}
		break;

	case NeedsCheckout:
		{
			FText ErrorMessage;

			if (!SourceControlHelpers::CheckoutOrMarkForAdd(TargetFilename, FText::FromString(TargetFilename), NULL, ErrorMessage))
			{
				FNotificationInfo Info(ErrorMessage);
				Info.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
			UpdateCache(false);
		}
		break;

	case ReadyToModify:
	default:
		break;
	}

	return FReply::Handled();
}

bool SPlatformSetupMessage::IsReadyToGo() const
{
	return CachedSetupState == ReadyToModify;
}

TAttribute<bool> SPlatformSetupMessage::GetReadyToGoAttribute() const
{
	return TAttribute<bool>(this, &SPlatformSetupMessage::IsReadyToGo);
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
