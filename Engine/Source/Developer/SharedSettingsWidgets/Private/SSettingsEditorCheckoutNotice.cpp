// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSettingsEditorCheckoutNotice.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Images/SThrobber.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SSettingsEditorCheckoutNotice"

namespace SettingsHelpers
{
	bool IsCheckedOut(const FString& InFileToCheckOut, bool bForceSourceControlUpdate)
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(InFileToCheckOut, bForceSourceControlUpdate ? EStateCacheUsage::ForceUpdate : EStateCacheUsage::Use);
			return SourceControlState.IsValid() && (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded());
		}

		return false;
	}

	bool IsSourceControlled(const FString& InFileToCheckOut, bool bForceSourceControlUpdate)
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(InFileToCheckOut, bForceSourceControlUpdate ? EStateCacheUsage::ForceUpdate : EStateCacheUsage::Use);
			return SourceControlState.IsValid() && SourceControlState->IsSourceControlled();
		}

		return false;
	}

	bool CheckOutOrAddFile(const FString& InFileToCheckOut, bool bForceSourceControlUpdate, bool ShowErrorInNotification, FText* OutErrorMessage)
	{
		FText ErrorMessage;
		bool bSuccessfullyCheckedOutOrAddedFile = false;
		if(ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(InFileToCheckOut, bForceSourceControlUpdate ? EStateCacheUsage::ForceUpdate : EStateCacheUsage::Use);

			TArray<FString> FilesToBeCheckedOut;
			FilesToBeCheckedOut.Add(InFileToCheckOut);

			if(SourceControlState.IsValid())
			{
				if (SourceControlState->IsSourceControlled())
				{
					if (SourceControlState->IsDeleted())
					{
						ErrorMessage = LOCTEXT("ConfigFileMarkedForDeleteError", "Error: The configuration file is marked for deletion.");
					}
					// Note: Here we attempt to check out files that are read only even if the internal state says they cannot be checked out.  This is to work around cases were the file is reverted or checked in and the internal state has not been updated yet
					else if (SourceControlState->CanCheckout() || SourceControlState->IsCheckedOutOther() || FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*InFileToCheckOut))
					{
						ECommandResult::Type CommandResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut);
						if (CommandResult == ECommandResult::Failed)
						{
							ErrorMessage = LOCTEXT("FailedToCheckOutConfigFileError", "Error: Failed to check out the configuration file.");
						}
						else if (CommandResult == ECommandResult::Cancelled)
						{
							ErrorMessage = LOCTEXT("CancelledCheckOutConfigFile", "Checkout was cancelled.  File will be marked writable.");
						}
						else
						{
							bSuccessfullyCheckedOutOrAddedFile = true;
						}
					}
				}
				else if (!SourceControlState->IsUnknown()) // most likely not source controled, so we'll try add it.
				{	
					if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*InFileToCheckOut))
					{
						// Hasn't been created yet
						return true;
					}

					ECommandResult::Type CommandResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeCheckedOut);

					if (CommandResult == ECommandResult::Failed)
					{
						ErrorMessage = LOCTEXT("FailedToAddConfigFileError", "Error: Failed to add the configuration file.");
					}
					else if (CommandResult == ECommandResult::Cancelled)
					{
						ErrorMessage = LOCTEXT("CancelledAddConfigFile", "Add was cancelled.  File will be marked writable.");
					}
					else
					{
						bSuccessfullyCheckedOutOrAddedFile = true;
					}
				}				
			}
		}

		if (!ErrorMessage.IsEmpty())
		{
			if (OutErrorMessage != nullptr)
			{
				*OutErrorMessage = ErrorMessage;
			}

			if (ShowErrorInNotification)
			{
				// Show a notification that the file could not be checked out
				FNotificationInfo CheckOutError(ErrorMessage);
				CheckOutError.ExpireDuration = 3.0f;
				FSlateNotificationManager::Get().AddNotification(CheckOutError);
			}
		}

		return bSuccessfullyCheckedOutOrAddedFile;
	}

	bool MakeWritable(const FString& InFileToMakeWritable, bool ShowErrorInNotification, FText* OutErrorMessage)
	{
		if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*InFileToMakeWritable))
		{
			return true;
		}

		bool bSuccess = FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*InFileToMakeWritable, false);
		if(!bSuccess)
		{
			FText ErrorMessage = FText::Format(LOCTEXT("FailedToMakeWritable", "Could not make {0} writable."), FText::FromString(InFileToMakeWritable));;

			if (OutErrorMessage != nullptr)
			{
				*OutErrorMessage = ErrorMessage;
			}

			if (ShowErrorInNotification)
			{
				FNotificationInfo MakeWritableNotification(ErrorMessage);
				MakeWritableNotification.ExpireDuration = 3.0f;

				FSlateNotificationManager::Get().AddNotification(MakeWritableNotification);
			}
		}

		return bSuccess;
	}
}
/* SSettingsEditorCheckoutNotice interface
 *****************************************************************************/

void SSettingsEditorCheckoutNotice::Construct( const FArguments& InArgs )
{
	OnFileProbablyModifiedExternally = InArgs._OnFileProbablyModifiedExternally;
	ConfigFilePath = InArgs._ConfigFilePath;

	LastDefaultConfigCheckOutTime = 0.0;
	DefaultConfigCheckOutNeeded = false;
	DefaultConfigQueryInProgress = true;

	const int Padding = 8;
	// default configuration notice
	ChildSlot
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex(this, &SSettingsEditorCheckoutNotice::HandleNoticeSwitcherWidgetIndex)

		// Locked slot
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)

			// Locked icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(Padding)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Lock"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			// Locked notice
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(FMargin(0.f, Padding, Padding, Padding))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SSettingsEditorCheckoutNotice::HandleLockedStatusText)
				.ColorAndOpacity(this, &SSettingsEditorCheckoutNotice::GetLockedStatusTextColor)
			]

			// Check out button
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.OnClicked(this, &SSettingsEditorCheckoutNotice::HandleCheckOutButtonClicked)
				.Text(this, &SSettingsEditorCheckoutNotice::HandleCheckOutButtonText)
				.ToolTipText(this, &SSettingsEditorCheckoutNotice::HandleCheckOutButtonToolTip)
				.Visibility(this, &SSettingsEditorCheckoutNotice::HandleCheckOutButtonVisibility)
			]
			// Source control status throbber
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 1.0f)
			[
				SNew(SThrobber)
				.Visibility(this, &SSettingsEditorCheckoutNotice::HandleThrobberVisibility)
			]
		]
		// Unlocked slot
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)

			// Unlocked icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(Padding)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Unlock"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			// Unlocked notice
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.f, Padding, Padding, Padding))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SSettingsEditorCheckoutNotice::HandleUnlockedStatusText)
			]
		]
	];
}


/* SSettingsEditorCheckoutNotice callbacks
 *****************************************************************************/

FReply SSettingsEditorCheckoutNotice::HandleCheckOutButtonClicked()
{
	FString TargetFilePath = ConfigFilePath.Get();

	bool bSuccess = SettingsHelpers::CheckOutOrAddFile(TargetFilePath);
	if (!bSuccess)
	{
		bSuccess = SettingsHelpers::MakeWritable(TargetFilePath);
	}
		
	if (bSuccess)
	{
		DefaultConfigCheckOutNeeded = false;
	}

	return FReply::Handled();
}


FText SSettingsEditorCheckoutNotice::HandleCheckOutButtonText( ) const
{
	if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		return LOCTEXT("CheckOutFile", "Check Out File");
	}

	return LOCTEXT("MakeWritable", "Make Writable");
}


FText SSettingsEditorCheckoutNotice::HandleCheckOutButtonToolTip( ) const
{
	if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		return LOCTEXT("CheckOutFileTooltip", "Check out the default configuration file that holds these settings.");
	}

	return LOCTEXT("MakeWritableTooltip", "Make the default configuration file that holds these settings writable.");
}

EVisibility SSettingsEditorCheckoutNotice::HandleCheckOutButtonVisibility() const
{
	// Display for checking out the file, or for making writable
	if ((ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable() && !DefaultConfigQueryInProgress) || 
		(!ISourceControlModule::Get().IsEnabled() && (DefaultConfigQueryInProgress || DefaultConfigCheckOutNeeded)))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FText SSettingsEditorCheckoutNotice::HandleLockedStatusText() const
{
	FText ConfigFilename = FText::FromString(FPaths::GetCleanFilename(ConfigFilePath.Get()));

	if (DefaultConfigQueryInProgress)
	{
		return FText::Format(LOCTEXT("DefaultSettingsNotice_Source", "These settings are saved in {0}. Checking file state..."), ConfigFilename);
	}
	
	return FText::Format(ISourceControlModule::Get().IsEnabled() ?
		LOCTEXT("DefaultSettingsNotice_WithSourceControl", "These settings are saved in {0}, which is currently NOT checked out.") :
		LOCTEXT("DefaultSettingsNotice_NotWritable", "These settings are saved in {0}, which is currently NOT writable."), ConfigFilename);
}

FText SSettingsEditorCheckoutNotice::HandleUnlockedStatusText() const
{
	FText ConfigFilename = FText::FromString(FPaths::GetCleanFilename(ConfigFilePath.Get()));
	
	return FText::Format(ISourceControlModule::Get().IsEnabled() ?
		LOCTEXT("DefaultSettingsNotice_CheckedOut", "These settings are saved in {0}, which is currently checked out.") :
		LOCTEXT("DefaultSettingsNotice_Writable", "These settings are saved in {0}, which is currently writable."), ConfigFilename);
}


EVisibility SSettingsEditorCheckoutNotice::HandleThrobberVisibility() const
{
	if(ISourceControlModule::Get().IsEnabled())
	{
		return DefaultConfigQueryInProgress ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

FSlateColor SSettingsEditorCheckoutNotice::GetLockedStatusTextColor() const
{
	static const FName Foreground("Colors.Foreground");
	static const FName Warning("Colors.AccentYellow");

	return (IsUnlocked() || DefaultConfigQueryInProgress) ? FAppStyle::Get().GetSlateColor(Foreground) : FAppStyle::Get().GetSlateColor(Warning);
}


void SSettingsEditorCheckoutNotice::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// cache selected settings object's configuration file state (purposely not in an active timer to make sure it appears while the user is active)
	if (InCurrentTime - LastDefaultConfigCheckOutTime >= 1.0f)
	{
		bool NewCheckOutNeeded = false;

		DefaultConfigQueryInProgress = true;
		FString CachedConfigFileName = ConfigFilePath.Get();
		if (!CachedConfigFileName.IsEmpty())
		{
			if (ISourceControlModule::Get().IsEnabled())
			{
				// note: calling QueueStatusUpdate often does not spam status updates as an internal timer prevents this
				ISourceControlModule::Get().QueueStatusUpdate(CachedConfigFileName);

				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(CachedConfigFileName, EStateCacheUsage::Use);
				NewCheckOutNeeded = SourceControlState.IsValid() && SourceControlState->CanCheckout();
				DefaultConfigQueryInProgress = SourceControlState.IsValid() && SourceControlState->IsUnknown();
			}
			else
			{
				NewCheckOutNeeded = (FPaths::FileExists(CachedConfigFileName) && IFileManager::Get().IsReadOnly(*CachedConfigFileName));
				DefaultConfigQueryInProgress = false;
			}

			// file has been checked in or reverted
			if ((NewCheckOutNeeded == true) && (DefaultConfigCheckOutNeeded == false))
			{
				OnFileProbablyModifiedExternally.ExecuteIfBound();
			}
		}

		DefaultConfigCheckOutNeeded = NewCheckOutNeeded;
		LastDefaultConfigCheckOutTime = InCurrentTime;
	}
}


bool SSettingsEditorCheckoutNotice::IsUnlocked() const
{
	return !DefaultConfigCheckOutNeeded && !DefaultConfigQueryInProgress;
}

#undef LOCTEXT_NAMESPACE


