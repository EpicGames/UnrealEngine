// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderFormatsPropertyDetails.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorDirectories.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "IDetailPropertyRow.h"
#include "RHI.h"

#define LOCTEXT_NAMESPACE "ShaderFormatsPropertyDetails"

FShaderFormatsPropertyDetails::FShaderFormatsPropertyDetails(IDetailLayoutBuilder* InDetailBuilder, const TCHAR* InProperty, const TCHAR* InTitle)
: DetailBuilder(InDetailBuilder)
, Property(InProperty)
, Title(InTitle)
{
	ShaderFormatsPropertyHandle = DetailBuilder->GetProperty(*Property);
	ensure(ShaderFormatsPropertyHandle.IsValid());
}

void FShaderFormatsPropertyDetails::SetOnUpdateShaderWarning(FSimpleDelegate const& Delegate)
{
	ShaderFormatsPropertyHandle->SetOnPropertyValueChanged(Delegate);
}

void FShaderFormatsPropertyDetails::CreateTargetShaderFormatsPropertyView(
	ITargetPlatformSettings* TargetPlatformSettings,
	GetFriendlyNameFromRHINameFnc FriendlyNameFnc,
	FilterShaderPlatformFnc* FilterShaderPlatformFunc,
	ECategoryPriority::Type Priority)
{
	check(TargetPlatformSettings);
	DetailBuilder->HideProperty(ShaderFormatsPropertyHandle);
	
	// List of supported RHI's and selected targets
	TArray<FName> ShaderFormats;
	TargetPlatformSettings->GetAllPossibleShaderFormats(ShaderFormats);
	
	IDetailCategoryBuilder& TargetedRHICategoryBuilder = DetailBuilder->EditCategory(Title, FText::GetEmpty(), Priority);
	
	int32 ShaderCounter = 0;
	for (const FName& ShaderFormat : ShaderFormats)
	{
		if (FilterShaderPlatformFunc && !FilterShaderPlatformFunc(ShaderFormat))
		{
			continue;
		}

		const FText FriendlyShaderFormatName = FriendlyNameFnc(ShaderFormat);
		if (!FriendlyShaderFormatName.IsEmpty())
		{
			ShaderFormatOrder.Add(ShaderFormat, ShaderCounter++);
			FDetailWidgetRow& TargetedRHIWidgetRow = TargetedRHICategoryBuilder.AddCustomRow(FriendlyShaderFormatName);

			TargetedRHIWidgetRow
			.NameContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0, 1, 0, 1))
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text(FriendlyShaderFormatName)
					.Font(DetailBuilder->GetDetailFont())
				]
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FShaderFormatsPropertyDetails::OnTargetedRHIChanged, ShaderFormat)
				.IsChecked(this, &FShaderFormatsPropertyDetails::IsTargetedRHIChecked, ShaderFormat)
			];
		}
	}
    
    ShaderFormatsPropertyHandle->GetProperty()->SetMetaData("ConfigRestartRequired", "true");
}


void FShaderFormatsPropertyDetails::OnTargetedRHIChanged(ECheckBoxState InNewValue, FName InRHIName)
{
	TArray<void*> RawPtrs;
	ShaderFormatsPropertyHandle->AccessRawData(RawPtrs);
	
	// Update the CVars with the selection
	{
		ShaderFormatsPropertyHandle->NotifyPreChange();
		for (void* RawPtr : RawPtrs)
		{
			TArray<FString>& Array = *(TArray<FString>*)RawPtr;
			if(InNewValue == ECheckBoxState::Checked)
			{
				// Preserve order from GetAllPossibleShaderFormats
				const int32 InIndex = ShaderFormatOrder[InRHIName];
				int32 InsertIndex = 0;
				for (; InsertIndex < Array.Num(); ++InsertIndex)
				{
					const int32* ShaderFormatIndex = ShaderFormatOrder.Find(*Array[InsertIndex]);
					if (ShaderFormatIndex != nullptr && InIndex < *ShaderFormatIndex) 
					{
						break;
					}
				}
				Array.Insert(InRHIName.ToString(), InsertIndex);
			}
			else
			{
				Array.Remove(InRHIName.ToString());
			}
		}

		ShaderFormatsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayMove);
	}
}


ECheckBoxState FShaderFormatsPropertyDetails::IsTargetedRHIChecked(FName InRHIName) const
{
	ECheckBoxState CheckState = ECheckBoxState::Unchecked;
	
	TArray<void*> RawPtrs;
	ShaderFormatsPropertyHandle->AccessRawData(RawPtrs);
	
	for(void* RawPtr : RawPtrs)
	{
		TArray<FString>& Array = *(TArray<FString>*)RawPtr;
		if(Array.Contains(InRHIName.ToString()))
		{
			CheckState = ECheckBoxState::Checked;
		}
	}
	return CheckState;
}

#undef LOCTEXT_NAMESPACE
