// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FileHelper.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

/////////////////////////////////////////////////////
// FManifestUpdateHelper

// This is a utility class used to update individual sections of an XML manifest or .plist in raw string form.
// It is formatting sensitive and will fail if things are formatted in an unexpected manner.
// This is a stopgap measure, and will be replaced using a proper XML parser when it is ready for use.
class FManifestUpdateHelper
{
public:
	UE_API FManifestUpdateHelper(const FString& InFilename);

	// Replaces the text in InOutString between MatchPrefix and MatchSuffix with NewInfix, returning true if it was found
	static UE_API bool ReplaceStringPortion(FString& InOutString, const FString& MatchPrefix, const FString& MatchSuffix, const FString& NewInfix);

	// Checks if a key exists matching MatchPrefix
	UE_API bool HasKey(const FString& MatchPrefix);

	// Replace a key in the manifest (between MatchPrefix and MatchSuffix) with NewInfix
	UE_API void ReplaceKey(const FString& MatchPrefix, const FString& MatchSuffix, const FString& NewInfix);

	// Finalizes the updater, and writes it back to the file, returning true if successful, and false if there were any errors
	UE_API bool Finalize(const FString& TargetFilename, bool bShowNotifyOnFailure = true, FFileHelper::EEncodingOptions EncodingOption = FFileHelper::EEncodingOptions::AutoDetect);

	// Returns the first error message
	FText GetFirstErrorMessage() const { return FirstErrorMessage; }

private:
	UE_API void WriteError(FText NewError);
private:
	FText FirstErrorMessage;
	FString ManifestString;
	bool bManifestDirty;
};

#undef UE_API
