// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/////////////////////////////////////////////////////
// FPlatformIconInfo

struct FPlatformIconInfo
{
	enum ERequiredState
	{
		Required,
		Optional
	};

	FString IconPath;
	FText IconName;
	FText IconDescription;
	FIntPoint IconRequiredSize;
	ERequiredState RequiredState;

	FPlatformIconInfo(const FString& InIconPath, FText InIconName, FText InIconDescription, int32 RequiredWidth = -1, int32 RequiredHeight = -1, ERequiredState IsRequired = Optional)
		: IconPath(InIconPath)
		, IconName(InIconName)
		, IconDescription(InIconDescription)
		, IconRequiredSize(RequiredWidth, RequiredHeight)
		, RequiredState(IsRequired)
	{

	}
};
