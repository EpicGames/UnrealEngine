// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyHandle.h"
#include "IExternalImagePickerModule.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

/**
 * Delegate fired before an image has been copied
 * @param	InChosenImage	The path to the image that has been picked by the user.
 * @return true if the operation was successful
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnPreExternalImageCopy, const FString& /** InChosenImage */);

/**
 * Delegate fired after an image has been copied
 * @param	InChosenImage	The path to the image that has been picked by the user.
 * @return true if the operation was successful
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnPostExternalImageCopy, const FString& /** InChosenImage */);

/////////////////////////////////////////////////////
// SExternalImageReference

// This widget shows an external image preview of a per-project configurable image
// (one where the engine provides a default, but each project may have its own override)

class SExternalImageReference : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SExternalImageReference)
		: _FileDescription(NSLOCTEXT("SExternalImageReference", "FileDescription", "External Image"))
		, _MaxDisplaySize(400.0f, 400.0f)
		, _RequiredSize(-1, -1)
		, _DeleteTargetWhenDefaultChosen(false)
		, _GenerateImageVisibility(EVisibility::Hidden)
		, _GenerateImageToolTipText(NSLOCTEXT("SExternalImageReference", "GenerateImage", "Generate Image"))
		{}

		/** The description of the file, used in error messages/notifications */
		SLATE_ARGUMENT(FText, FileDescription)

		/** How big should we display the image? */
		SLATE_ARGUMENT(FVector2D, MaxDisplaySize)

		/** How big does the image need to be (any size is allowed if this is omitted) */
		SLATE_ARGUMENT(FIntPoint, RequiredSize)

		/** Delegate fired before an image has been copied */
		SLATE_ARGUMENT(FOnPreExternalImageCopy, OnPreExternalImageCopy)

		/** Delegate fired after an image has been copied */
		SLATE_ARGUMENT(FOnPostExternalImageCopy, OnPostExternalImageCopy)

		/** Delegate fired to get the path to start picking from. */
		SLATE_ARGUMENT(FOnGetPickerPath, OnGetPickerPath)

		/** A property handle to use if required */
		SLATE_ARGUMENT(TSharedPtr<class IPropertyHandle>, PropertyHandle)

		/** If true, the target image will be deleted if the default is chosen */
		SLATE_ARGUMENT(bool, DeleteTargetWhenDefaultChosen)

		/** File extensions allowed for the external image reference */
		SLATE_ARGUMENT(TArray<FString>, FileExtensions)

		/** If true, deletes the previous reference if the file extension changes */
		SLATE_ARGUMENT(bool, DeletePreviousTargetWhenExtensionChanges)

		/** Whether to show the generate image button */
		SLATE_ATTRIBUTE(EVisibility, GenerateImageVisibility)

		/** Tooltip for Generate Image button */
		SLATE_ARGUMENT(FText, GenerateImageToolTipText);

		/** Delegate fired when an Generate Image button is pressed */
		SLATE_EVENT(FOnClicked, OnGenerateImageClicked)
    
	SLATE_END_ARGS()

public:

	UE_API void Construct(const FArguments& InArgs, const FString& InBaseFilename, const FString& InOverrideFilename);

protected:
	/** Delegate handler for when an image is picked */
	UE_API bool HandleExternalImagePicked(const FString& InChosenImage, const FString& InTargetImage);

protected:

	/** The image on disk that we will use if the override does not exist. */
	FString BaseFilename;

	/** The image on disk that the override image is stored as. */
	FString OverrideFilename;

	/** The description of the file in question, e.g. 'image' or 'icon', used for error reporting */
	FText FileDescription;

	/** Delegate fired before an image has been copied */
	FOnPreExternalImageCopy OnPreExternalImageCopy;

	/**  Delegate fired after an image has been copied */
	FOnPostExternalImageCopy OnPostExternalImageCopy;

	/** A property handle to use if required */
	TSharedPtr<class IPropertyHandle> PropertyHandle;

	/** If true, the target image will be deleted if the default is chosen */
	bool bDeleteTargetWhenDefaultChosen;

	/** The extensions supported by this external reference */
	TArray<FString> Extensions;

	/** If true, the previous target image will be deleted if the file extension changes */
	bool bDeletePreviousTargetWhenExtensionChanges;
};

#undef UE_API
