// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

/////////////////////////////////////////////////////
// SHyperlinkLaunchURL

// This widget is a hyperlink that launches an external URL when clicked

class SHyperlinkLaunchURL : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SHyperlinkLaunchURL)
		{}

		/** If set, this text will be used for the display string of the hyperlink */
		SLATE_ATTRIBUTE(FText, Text)

	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, const FString& InDestinationURL);

protected:
	UE_API void OnNavigate();

protected:
	FString DestinationURL;
};

#undef UE_API
