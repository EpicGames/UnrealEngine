// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "ISourceControlProvider.h"
#include "ISourceControlOperation.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

/////////////////////////////////////////////////////
// SPlatformSetupMessage

// This widget displays a setup message indicating if the game project is configured for a platform or not

class SPlatformSetupMessage : public SCompoundWidget
{
	enum ESetupState
	{
		MissingFiles,
		NeedsCheckout,
		ReadOnlyFiles,
		ReadyToModify,
		GettingStatus
	};

	SLATE_BEGIN_ARGS(SPlatformSetupMessage)
		{}

		// Name of the platform
		SLATE_ARGUMENT(FText, PlatformName)

		// Called when the Setup button is clicked
		SLATE_EVENT(FSimpleDelegate, OnSetupClicked)

	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, const FString& InTargetFilename);

	UE_API TAttribute<bool> GetReadyToGoAttribute() const;

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of Swidget interface
private:
	UE_API int32 GetSetupStateAsInt() const;
	UE_API bool IsReadyToGo() const;
	UE_API FSlateColor GetBorderColor() const;

	UE_API TSharedRef<SWidget> MakeRow(FName IconName, FText Message, FText ButtonMessage);

	UE_API FReply OnButtonPressed();

	UE_API void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	// Returns the setup state of a specified file
	UE_API ESetupState GetSetupStateBasedOnFile(bool bInitStatus);

	// Updates the cache CachedSetupState 
	UE_API void UpdateCache(bool bForceUpdate);
private:
	FString TargetFilename;
	ESetupState CachedSetupState;
	FSimpleDelegate OnSetupClicked;
};

#undef UE_API
