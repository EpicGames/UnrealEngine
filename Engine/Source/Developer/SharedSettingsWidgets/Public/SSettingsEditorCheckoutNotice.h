// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

namespace SettingsHelpers
{
	SHAREDSETTINGSWIDGETS_API bool IsSourceControlled(const FString& InFileToCheckOut, bool bForceSourceControlUpdate = false);
	SHAREDSETTINGSWIDGETS_API bool IsCheckedOut(const FString& InFileToCheckOut, bool bForceSourceControlUpdate = false);
	SHAREDSETTINGSWIDGETS_API bool CheckOutOrAddFile(const FString& InFileToCheckOut, bool bForceSourceControlUpdate = false, bool ShowErrorInNotification = true, FText* OutErrorMessage = nullptr);
	SHAREDSETTINGSWIDGETS_API bool MakeWritable(const FString& InFileToMakeWritable, bool ShowErrorInNotification = true, FText* OutErrorMessage = nullptr);
}

/**
 * Implements a widget that provides a notice for files that need to be checked out.
 */
class SSettingsEditorCheckoutNotice : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSettingsEditorCheckoutNotice) { }

		/** Called to get the filename of the config file for display */
		SLATE_ATTRIBUTE(FString, ConfigFilePath)

		/** Called when the the file was probably modified externally (only detects files that were 'ready to go' and are now not, e.g., reverted/checked in or files with the read-only flag edited outside) */
		SLATE_EVENT(FSimpleDelegate, OnFileProbablyModifiedExternally)

	SLATE_END_ARGS()

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	UE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	// Return true if the watched file can be modified (as of the last update)
	UE_API bool IsUnlocked() const;

	// Force an update on the next tick
	void Invalidate()
	{
		LastDefaultConfigCheckOutTime = 0.f;
	}

private:

	// Callback for clicking the 'Check Out' button.
	UE_API FReply HandleCheckOutButtonClicked( );

	// Callback for getting the text of the 'Check Out' button.
	UE_API FText HandleCheckOutButtonText( ) const;

	// Callback for getting the tool tip text of the 'Check Out' button.
	UE_API FText HandleCheckOutButtonToolTip( ) const;

	// Callback for determining the visibility of the check-out button.
	UE_API EVisibility HandleCheckOutButtonVisibility( ) const;

	// Callback for getting the widget index for the notice switcher.
	int32 HandleNoticeSwitcherWidgetIndex( ) const
	{
		return IsUnlocked() ? 1 : 0;
	}

	// Callback for getting the status text when the config is locked
	UE_API FText HandleLockedStatusText() const;

	// Callback for getting the status text when the config is unlocked
	UE_API FText HandleUnlockedStatusText() const;

	// Callback for getting the visibility of the source control throbber
	UE_API EVisibility HandleThrobberVisibility() const;

	// Callback to get the background color of the checkout wanring
	UE_API FSlateColor GetLockedStatusTextColor() const;
private:

	// Called when the file was modified externally
	FSimpleDelegate OnFileProbablyModifiedExternally;

	// The current file being watched
	TAttribute<FString> ConfigFilePath;

	// Holds a flag indicating whether the section's configuration file needs to be checked out.
	bool DefaultConfigCheckOutNeeded;

	// Holds a flag indicating whether the section's configuration file has a source control status request in progress.
	bool DefaultConfigQueryInProgress;

	// Holds a timer for checking whether the section's configuration file needs to be checked out.
	double LastDefaultConfigCheckOutTime;
};

#undef UE_API
