// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailLayoutBuilder.h"

#define UE_API SHAREDSETTINGSWIDGETS_API

class IDetailLayoutBuilder;
class IPropertyHandle;
class ITargetPlatform;
class ITargetPlatformSettings;

enum class ECheckBoxState : uint8;

/**
 * Helper which implements details panel customizations for a device profiles parent property
 */
class FShaderFormatsPropertyDetails
: public TSharedFromThis<FShaderFormatsPropertyDetails>
{
public:
	typedef FText(Deprecated_GetFriendlyNameFromRHINameFnc)(const FString&);

	typedef FText (GetFriendlyNameFromRHINameFnc)(FName);
	typedef bool (FilterShaderPlatformFnc)(FName);
	
	/**
	 * Constructor for the parent property details view
	 *
	 * @param InDetailsBuilder - Where we are adding our property view to
	 * @param InProperty - The category name to override
	 * @param InTitle - Title for display
	 */
	UE_API FShaderFormatsPropertyDetails(IDetailLayoutBuilder* InDetailBuilder, const TCHAR* InProperty = TEXT("TargetedRHIs"), const TCHAR* InTitle = TEXT("Targeted RHIs"));

	/** Simple delegate for updating shader version warning */
	UE_API void SetOnUpdateShaderWarning(const FSimpleDelegate& Delegate);
	
	/** Create the UI to select which windows shader formats we are targeting */
	UE_API void CreateTargetShaderFormatsPropertyView(ITargetPlatformSettings* TargetPlatform, GetFriendlyNameFromRHINameFnc* FriendlyNameFnc, FilterShaderPlatformFnc* FilterShaderPlatformFunc = nullptr, ECategoryPriority::Type InPriority = ECategoryPriority::Default);
	
	/**
	 * @param InRHIName - The input RHI to check
	 * @returns Whether this RHI is currently enabled
	 */
	UE_API ECheckBoxState IsTargetedRHIChecked(FName InRHIName) const;
	
private:
	
	// Supported/Targeted RHI check boxes
	UE_API void OnTargetedRHIChanged(ECheckBoxState InNewValue, FName InRHIName);
	
private:
	
	/** A handle to the detail view builder */
	IDetailLayoutBuilder* DetailBuilder;
	
	/** Access to the Parent Property */
	TSharedPtr<IPropertyHandle> ShaderFormatsPropertyHandle;

	/** The category name to override */
	FString Property;


	/** Title for display */
	FName Title;

	/** Preserve shader format order when writing to property */
	TMap<FName, int> ShaderFormatOrder;
};

#undef UE_API
