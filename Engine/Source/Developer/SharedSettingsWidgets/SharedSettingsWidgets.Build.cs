// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SharedSettingsWidgets : ModuleRules
{
	public SharedSettingsWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				
				"RHI",
				"SourceControl",
				"TargetPlatform",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ExternalImagePicker",
			}
		);
	}
}
